{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"SupportedTargetPlatforms": ["WinGDK"],
	"Modules": [
		{
			"Name": "GDKPackageChunkInstall",
			"Type": "Runtime",
			"PlatformAllowList": ["WinGDK"]
		}
	]
}