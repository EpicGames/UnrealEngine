{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "GDK Package Chunk Installer",
	"Description": "Allows titles to use GDK chunk installer (XPackage)",
	"CreatedBy": "Epic Games, Inc.",
	"Category": "Platform",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"RequiresBuildPlatform": false,
	"CanContainContent": false,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64" ],
	"Modules": [
		{
			"Name": "GDKPackageChunkInstall",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins":
	[
		{
			"Name": "MSGameStore",
			"Enabled": true,
			"PlatformAllowList": [ "Win64" ]
		},
		{
			"Name": "MSGamingRuntime",
			"Enabled": true,
			"PlatformAllowList": [ "Win64" ]
		}
	]
}