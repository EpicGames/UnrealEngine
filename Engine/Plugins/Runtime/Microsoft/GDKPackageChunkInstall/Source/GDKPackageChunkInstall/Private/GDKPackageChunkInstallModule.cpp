// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "Modules/ModuleManager.h"
#include "GDKPlatformChunkInstall.h"
#include "IGDKPackageManifestModule.h"
#include "GDKThreadCheck.h"
#include "GDKTaskQueueHelpers.h"
#include "GDKRuntimeModule.h"
#include "HAL/IConsoleManager.h"

#if WITH_GRDK
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XPackage.h>
#include <XGameErr.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

// not relevent for general use
#define WITH_XPACKAGE_DEBUG_CMDS 0


/**
 * Module for the GDK Package Chunk Installer
 */
class FGDKPackageChunkInstallModule : public IPlatformChunkInstallModule
{
public:
	std::atomic<bool> bWantSystemCreation;
	TUniquePtr<FGDKPlatformChunkInstall> ChunkInstaller;

	FGDKPackageChunkInstallModule()
		: bWantSystemCreation(true)
	{
#if WITH_EDITOR
		// tear down the chunk installer when PIE finishes
		IGDKRuntimeModule::Get().GetOnTeardownForPIE().AddRaw( this, &FGDKPackageChunkInstallModule::OnTeardownForPIE );
#endif

		// ensure the package manifest module is loaded if we'll need it asyncronously later on
#if WITH_CHUNKINSTALL_ASYNC_INIT
		IGDKPackageManifestModule::Get();
#endif
	}

	virtual void ShutdownModule() override
	{
		ChunkInstaller.Reset();
	
#if WITH_EDITOR
		if (IGDKRuntimeModule* GDKRuntime = IGDKRuntimeModule::TryGet())
		{
			GDKRuntime->GetOnTeardownForPIE().RemoveAll(this);
		}
#endif
	}

	virtual IPlatformChunkInstall* GetPlatformChunkInstall()
	{
		// prevent creation of GDK chunk installer if the GDK runtime is unavailable
		if (!IGDKRuntimeModule::Get().IsAvailable())
		{
			bWantSystemCreation = false;
		}

		if (bWantSystemCreation)
		{
			bWantSystemCreation = false;
			ChunkInstaller.Reset(new FGDKPlatformChunkInstall());
		}

		return ChunkInstaller.Get();
	}

#if WITH_EDITOR
	void OnTeardownForPIE()
	{
		ChunkInstaller.Reset();
		bWantSystemCreation = true;
	}
#endif


#if WITH_XPACKAGE_DEBUG_CMDS
	static void CALLBACK DebugInstallXPackageCallback(void*, XPackageInstallationMonitorHandle Handle)
	{
		XPackageInstallationProgress Progress;
		XPackageGetInstallationProgress(Handle, &Progress);
		UE_LOGF(LogChunkInstaller, Log, "XPackage debug chunk installing %llu / %llu", Progress.installedBytes, Progress.totalBytes );
		if (Progress.completed)
		{
			UE_LOGF(LogChunkInstaller, Log, "XPackage debug chunk installed" );
			XPackageCloseInstallationMonitorHandle(Handle);
		}
	}

	void DebugInstallXPackageChunk( XPackageChunkSelector& Selector )
	{
		GDK_SCOPE_NOT_TIME_SENSITIVE(); // (debug only) XPackageInstallChunks is not safe to call on a time-sensitive thread

		char PackageIdentifier[XPACKAGE_IDENTIFIER_MAX_LENGTH];
		XPackageGetCurrentProcessPackageIdentifier(XPACKAGE_IDENTIFIER_MAX_LENGTH, PackageIdentifier);

		XPackageInstallationMonitorHandle Handle = nullptr;
		HRESULT hr = XPackageInstallChunks(PackageIdentifier, 1, &Selector, 100, true, FGDKAsyncTaskQueue::GetGenericQueue(), &Handle);
		if (SUCCEEDED(hr))
		{
			XTaskQueueRegistrationToken Token;
			hr = XPackageRegisterInstallationProgressChanged( Handle, nullptr, DebugInstallXPackageCallback, &Token);
			if (FAILED(hr))
			{
				XPackageCloseInstallationMonitorHandle(Handle);
			}
		}
		UE_CLOGF(FAILED(hr), LogChunkInstaller, Warning, "XPackageInstallChunks failed : 0x%X", hr );
	}

	void DebugUninstallXPackageChunk( XPackageChunkSelector& Selector )
	{
		char PackageIdentifier[XPACKAGE_IDENTIFIER_MAX_LENGTH];
		XPackageGetCurrentProcessPackageIdentifier(XPACKAGE_IDENTIFIER_MAX_LENGTH, PackageIdentifier);
		HRESULT hr = XPackageUninstallChunks(PackageIdentifier, 1, &Selector );
		UE_CLOGF(FAILED(hr), LogChunkInstaller, Warning, "XPackageUninstallChunks failed : 0x%X", hr );
	}
#endif //WITH_XPACKAGE_DEBUG_CMDS

};

#if !UE_BUILD_SHIPPING
static FAutoConsoleCommand CVarChunkInstallDumpChunks(
	TEXT("GDK.ChunkInstall.DumpChunks"),
	TEXT("Dumps the state of the chunk installation"),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		FGDKPackageChunkInstallModule* Module = FModuleManager::GetModulePtr<FGDKPackageChunkInstallModule>("GDKPackageChunkInstall");
		if (Module != nullptr && Module->ChunkInstaller.IsValid())
		{
			Module->ChunkInstaller->DebugDumpChunkState();
		}
	})
);


static FAutoConsoleCommand CVarChunkInstallDebugInstall(
	TEXT("GDK.ChunkInstall.InstallNamedChunk"),
	TEXT("[Name] Request installation of the given named chunk"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FGDKPackageChunkInstallModule* Module = FModuleManager::GetModulePtr<FGDKPackageChunkInstallModule>("GDKPackageChunkInstall");
		if (Module != nullptr && Module->ChunkInstaller.IsValid() && Args.Num() > 0)
		{
			TArray<FName> NamedChunks; 
			for (const FString& Arg : Args) NamedChunks.Add(FName(*Arg));
			Module->ChunkInstaller->InstallNamedChunks(NamedChunks);
		}
	})
);


static FAutoConsoleCommand CVarChunkUninstallDebugInstall(
	TEXT("GDK.ChunkInstall.UninstallNamedChunk"),
	TEXT("[Name] Request removal of the given named chunk"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FGDKPackageChunkInstallModule* Module = FModuleManager::GetModulePtr<FGDKPackageChunkInstallModule>("GDKPackageChunkInstall");
		if (Module != nullptr && Module->ChunkInstaller.IsValid() && Args.Num() > 0)
		{
			TArray<FName> NamedChunks; 
			for (const FString& Arg : Args) NamedChunks.Add(FName(*Arg));
			Module->ChunkInstaller->UninstallNamedChunks(NamedChunks);
		}
	})
);


static FAutoConsoleCommand GDKPackageDump(
	TEXT("GDK.XPackage.Dump"),
	TEXT("Dumps the details of all GDK XPackages"),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		GDK_SCOPE_NOT_TIME_SENSITIVE(); // (debug only) XPackageEnumeratePackages & XPackageEnumerateChunkAvailability are not safe to call on a time-sensitive thread

		auto EnumPackageCallback = [](void* Context, const XPackageDetails* Details)
		{
			//dump package details
			UE_LOGF(LogChunkInstaller, Log, "-------------" );
			UE_LOGF(LogChunkInstaller, Log, "Found %ls package:", Details->kind == XPackageKind::Content ? TEXT("Content") : TEXT("Game"));
			UE_LOGF(LogChunkInstaller, Log, "\tIdentifier:   %ls", *BytesToHex((uint8*)Details->packageIdentifier,XPACKAGE_IDENTIFIER_MAX_LENGTH) );
			UE_LOGF(LogChunkInstaller, Log, "\tVersion:      %d.%d.%d.%d", Details->version.major, Details->version.minor, Details->version.build, Details->version.revision );
			UE_LOGF(LogChunkInstaller, Log, "\tDisplay Name: %ls", Details->displayName ? UTF8_TO_TCHAR(Details->displayName) : TEXT("<none>") );
			UE_LOGF(LogChunkInstaller, Log, "\tDescription:  %ls", Details->description ? UTF8_TO_TCHAR(Details->description) : TEXT("<none>") );
			UE_LOGF(LogChunkInstaller, Log, "\tPublisher:    %ls", Details->publisher   ? UTF8_TO_TCHAR(Details->publisher)   : TEXT("<none>") );
			UE_LOGF(LogChunkInstaller, Log, "\tStoreId:      %ls", Details->storeId     ? UTF8_TO_TCHAR(Details->storeId)     : TEXT("<none>") );
			UE_LOGF(LogChunkInstaller, Log, "\tInstalling:   %ls", *LexToString(Details->installing) );

			// dump out the status of all known chunks in the package
			UE_LOGF(LogChunkInstaller, Log, "\tChunks:" );
			auto EnumChunkCallback = [](void* Context, const XPackageChunkSelector* Selector, XPackageChunkAvailability Availability)
			{
				UE_LOGF(LogChunkInstaller, Log, "\t\tChunk %-32ls %ls", *LexToString(*Selector), *LexToString(Availability) );
				return true;
			};
			XPackageEnumerateChunkAvailability( Details->packageIdentifier, XPackageChunkSelectorType::Chunk,    Context, EnumChunkCallback );
			XPackageEnumerateChunkAvailability( Details->packageIdentifier, XPackageChunkSelectorType::Language, Context, EnumChunkCallback );
			XPackageEnumerateChunkAvailability( Details->packageIdentifier, XPackageChunkSelectorType::Tag,      Context, EnumChunkCallback );
			XPackageEnumerateChunkAvailability( Details->packageIdentifier, XPackageChunkSelectorType::Feature,  Context, EnumChunkCallback );
			UE_LOGF(LogChunkInstaller, Log, "-------------" );
			return true;
		};

		// dump out the status of all known packages
		UE_LOGF(LogChunkInstaller, Log, "All XPackages:" );
		if (XPackageEnumeratePackages( XPackageKind::Game, XPackageEnumerationScope::ThisAndRelated, nullptr, EnumPackageCallback ) == E_GAMEPACKAGE_NO_STORE_ID)
		{
			// this title is not configured in the store yet, so just enumerate the current package
			UE_LOGF(LogChunkInstaller, Log, "(title has no store Id - just showing this package" );
			XPackageEnumeratePackages( XPackageKind::Game, XPackageEnumerationScope::ThisOnly, nullptr, EnumPackageCallback );
		}
		else
		{
			// enumerate other related content too, although it's not supported yet
			XPackageEnumeratePackages( XPackageKind::Content, XPackageEnumerationScope::ThisAndRelated, nullptr, EnumPackageCallback );
		}
	})
);
#endif //!UE_BUILD_SHIPPING

#if WITH_XPACKAGE_DEBUG_CMDS
static FAutoConsoleCommand GDKPackageInstallLanguage(
	TEXT("GDK.XPackage.InstallLanguage"),
	TEXT(" [lang] Install the given language chunk"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FGDKPackageChunkInstallModule* Module = FModuleManager::LoadModulePtr<FGDKPackageChunkInstallModule>("GDKPackageChunkInstall");
		if (Module != nullptr && Args.Num() == 1)
		{
			FTCHARToUTF8 UTF8Arg(*Args[0]);
			XPackageChunkSelector Selector;
			Selector.type = XPackageChunkSelectorType::Language;
			Selector.language = UTF8Arg.Get();
			Module->DebugInstallXPackageChunk(Selector);
		}
	})
);

static FAutoConsoleCommand GDKPackageInstallTag(
	TEXT("GDK.XPackage.InstallTag"),
	TEXT(" [tag] Install the given tagged chunk"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FGDKPackageChunkInstallModule* Module = FModuleManager::LoadModulePtr<FGDKPackageChunkInstallModule>("GDKPackageChunkInstall");
		if (Module != nullptr && Args.Num() == 1)
		{
			FTCHARToUTF8 UTF8Arg(*Args[0]);
			XPackageChunkSelector Selector;
			Selector.type = XPackageChunkSelectorType::Tag;
			Selector.tag = UTF8Arg.Get();
			Module->DebugInstallXPackageChunk(Selector);
		}
	})
);

static FAutoConsoleCommand GDKPackageInstallChunk(
	TEXT("GDK.XPackage.InstallChunk"),
	TEXT("[id] Install the given chunk id"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FGDKPackageChunkInstallModule* Module = FModuleManager::LoadModulePtr<FGDKPackageChunkInstallModule>("GDKPackageChunkInstall");
		if (Module != nullptr && Args.Num() == 1)
		{
			XPackageChunkSelector Selector;
			Selector.type = XPackageChunkSelectorType::Chunk;
			Selector.chunkId = FCString::Atoi(*Args[0]);
			Module->DebugInstallXPackageChunk(Selector);
		}
	})
);

static FAutoConsoleCommand GDKPackageInstallFeature(
	TEXT("GDK.XPackage.InstallFeature"),
	TEXT(" [feature] Install the given feature"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FGDKPackageChunkInstallModule* Module = FModuleManager::LoadModulePtr<FGDKPackageChunkInstallModule>("GDKPackageChunkInstall");
		if (Module != nullptr && Args.Num() == 1)
		{
			FTCHARToUTF8 UTF8Arg(*Args[0]);
			XPackageChunkSelector Selector;
			Selector.type = XPackageChunkSelectorType::Feature;
			Selector.feature = UTF8Arg.Get();
			Module->DebugInstallXPackageChunk(Selector);
		}
	})
);

static FAutoConsoleCommand GDKPackageUninstallLanguage(
	TEXT("GDK.XPackage.UninstallLanguage"),
	TEXT(" [lang] Uninstall the given language chunk"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FGDKPackageChunkInstallModule* Module = FModuleManager::LoadModulePtr<FGDKPackageChunkInstallModule>("GDKPackageChunkInstall");
		if (Module != nullptr && Args.Num() == 1)
		{
			FTCHARToUTF8 UTF8Arg(*Args[0]);
			XPackageChunkSelector Selector;
			Selector.type = XPackageChunkSelectorType::Language;
			Selector.language = UTF8Arg.Get();
			Module->DebugUninstallXPackageChunk(Selector);
		}
	})
);

static FAutoConsoleCommand GDKPackageUninstallTag(
	TEXT("GDK.XPackage.UninstallTag"),
	TEXT(" [tag] Uninstall the given tagged chunk"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FGDKPackageChunkInstallModule* Module = FModuleManager::LoadModulePtr<FGDKPackageChunkInstallModule>("GDKPackageChunkInstall");
		if (Module != nullptr && Args.Num() == 1)
		{
			FTCHARToUTF8 UTF8Arg(*Args[0]);
			XPackageChunkSelector Selector;
			Selector.type = XPackageChunkSelectorType::Tag;
			Selector.tag = UTF8Arg.Get();
			Module->DebugUninstallXPackageChunk(Selector);
		}
	})
);

static FAutoConsoleCommand GDKPackageUninstallChunk(
	TEXT("GDK.XPackage.UninstallChunk"),
	TEXT("[id] Uninstall the given chunk id"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FGDKPackageChunkInstallModule* Module = FModuleManager::LoadModulePtr<FGDKPackageChunkInstallModule>("GDKPackageChunkInstall");
		if (Module != nullptr && Args.Num() == 1)
		{
			XPackageChunkSelector Selector;
			Selector.type = XPackageChunkSelectorType::Chunk;
			Selector.chunkId = FCString::Atoi(*Args[0]);
			Module->DebugUninstallXPackageChunk(Selector);
		}
	})
);

static FAutoConsoleCommand GDKPackageUninstallFeature(
	TEXT("GDK.XPackage.UninstallFeature"),
	TEXT(" [feature] Uninstall the given feature"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FGDKPackageChunkInstallModule* Module = FModuleManager::LoadModulePtr<FGDKPackageChunkInstallModule>("GDKPackageChunkInstall");
		if (Module != nullptr && Args.Num() == 1)
		{
			FTCHARToUTF8 UTF8Arg(*Args[0]);
			XPackageChunkSelector Selector;
			Selector.type = XPackageChunkSelectorType::Feature;
			Selector.feature = UTF8Arg.Get();
			Module->DebugUninstallXPackageChunk(Selector);
		}
	})
);
#endif //WITH_XPACKAGE_DEBUG_CMDS

#else // ... WITH_GRDK

// stub
class FGDKPackageChunkInstallModule : public IPlatformChunkInstallModule
{
	virtual IPlatformChunkInstall* GetPlatformChunkInstall() override { return nullptr; }
};
#endif //WITH_GRDK



IMPLEMENT_MODULE(FGDKPackageChunkInstallModule, GDKPackageChunkInstall);

