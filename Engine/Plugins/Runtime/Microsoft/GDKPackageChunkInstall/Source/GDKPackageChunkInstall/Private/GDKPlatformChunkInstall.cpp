// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKPlatformChunkInstall.h"
#if WITH_GRDK
#include "GDKTaskQueueHelpers.h"
#include "GDKThreadCheck.h"
#include "IGDKPackageManifestModule.h"
#include "HAL/FileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "HAL/IConsoleManager.h"
#include "Async/TaskGraphInterfaces.h"
#include "Async/Async.h"
#include "Tasks/Task.h"

// default update interval for chunk installation in milliseconds
#if !defined(GDK_CHUNKINSTALL_UPDATEINTERVAL_MS)
	#define GDK_CHUNKINSTALL_UPDATEINTERVAL_MS 1000
#endif


static int32 GGDKChunkInstallSuppressUserConfirmation = 1;
static FAutoConsoleVariableRef CVarChunkInstallSuppressUserConfirmation(
	TEXT( "GDK.ChunkInstall.SuppressUserConfirmation" ),
	GGDKChunkInstallSuppressUserConfirmation,
	TEXT( "Whether to show a user confirmation when trying to download chunks over a certain size" ));

static bool GGDKChunkInstallTerminateGameOnExternalFeatureRemoval = true;
static FAutoConsoleVariableRef CVarChunkInstallTerminateGameOnExternalFeatureRemoval(
	TEXT("GDK.ChunkInstall.TerminateOnExternalFeatureRemoval"),
	GGDKChunkInstallTerminateGameOnExternalFeatureRemoval,
	TEXT("Whether to terminate the game if the user removes a Feature via the 'Manage Game & Add-Ins' UI")); // note: on Windows we should receive WM_CLOSE when a Feature is removed already

static bool GGDKChunkInstallEnableCancellation = true;
static FAutoConsoleVariableRef CVarChunkInstallEnableCancellation(
	TEXT("GDK.ChunkInstall.EnableCancellation"),
	GGDKChunkInstallEnableCancellation,
	TEXT("Whether cancellation of in-progress chunk downloads is enabled. When cancelled, any partially downloaded chunk data will be uninstalled."));


// chunk numbering is off by 2 to account for the registration chunk and binary chunk
static const int ChunkIDOffset = 2;
static inline uint32 ToUnrealChunkID( uint32 PackageChunkId )  { return PackageChunkId - ChunkIDOffset; }
static inline uint32 FromUnrealChunkID( uint32 UnrealChunkID ) { return UnrealChunkID + ChunkIDOffset; }



FGDKPlatformChunkInstall::FGDKPlatformChunkInstall()
{
	UE_LOGF(LogChunkInstaller, Display, "GDK platform chunk installer created");

	HRESULT hResult;

	// check whether this is an installed package build or not
	bIsPackaged = XPackageIsPackagedProcess();
	if (!bIsPackaged)
	{
		UE_LOGF( LogChunkInstaller, Log, "not a packaged process - no chunk installation required" );
		return;
	}

	// cache package identifier
	FMemory::Memzero(PackageIdentifier);
	hResult = XPackageGetCurrentProcessPackageIdentifier(XPACKAGE_IDENTIFIER_MAX_LENGTH, PackageIdentifier);
	UE_CLOGF( FAILED(hResult), LogChunkInstaller, Fatal, "Failed to get process package identifier. 0x%X", hResult );

#if !WITH_CHUNKINSTALL_ASYNC_INIT
#if USE_GDK_PACKAGE_MANIFEST_INIT
	CreateChunkDataFromPackageManifest();
#else
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (startup only) XPackageEnumeratePackages is not safe to call on a time-sensitive thread

	// create chunk data for the main package
	auto EnumeratePackageCallback = [](void* Context, const XPackageDetails* Details)
	{
		FGDKPlatformChunkInstall* This = (FGDKPlatformChunkInstall*)Context;
		This->CreateChunkDataForPackage(*Details);
		return true;
	};

	hResult = XPackageEnumeratePackages( XPackageKind::Game, XPackageEnumerationScope::ThisOnly, this, EnumeratePackageCallback );
	UE_CLOGF(FAILED(hResult), LogChunkInstaller, Error, "XPackageEnumeratePackages failed: 0x%X", hResult);	
#endif //USE_GDK_PACKAGE_MANIFEST_INIT
#endif //WITH_CHUNKINSTALL_ASYNC_INIT
}



FGDKPlatformChunkInstall::~FGDKPlatformChunkInstall()
{
	RemoveFromTicker();
}


bool FGDKPlatformChunkInstall::IsAvailable() const
{
	return bIsPackaged;
}





bool FGDKPlatformChunkInstall::MountPaks(uint32 UnrealChunkID)
{
	UE_LOGF(LogChunkInstaller, Log, "Mounting pak files from chunk %i.", UnrealChunkID);

	// find and mount all pak files
	const TArray<FString> PakchunkFiles = IGDKPackageManifestModule::Get().GetPakFilesInChunk(UnrealChunkID);
	bool bChunkMounted = true;
	for (const FString& PakchunkFile : PakchunkFiles)
	{
		FString PakLocation = TEXT("../../..") / PakchunkFile;
		PakLocation.ReplaceCharInline(TEXT('\\'), TEXT('/'));
		IPakFile* PakFile = FCoreDelegates::MountPak.IsBound() ? FCoreDelegates::MountPak.Execute(PakLocation, INDEX_NONE) : nullptr;

		if (PakFile)
		{
			UE_LOGF(LogChunkInstaller, Log, "Successfully mounted %ls", *PakchunkFile);
		}
		else
		{
			UE_LOGF(LogChunkInstaller, Log, "Failed to mount %ls", *PakchunkFile);
		}

		bChunkMounted &= (PakFile != nullptr);
	}

	TFunction<void()> BroadcastDelegateFunc = [this, bChunkMounted, UnrealChunkID]()
	{
		if (bChunkMounted)
		{
			InstallDelegate.Broadcast(UnrealChunkID, true);
		}
		else
		{
			UE_LOGF(LogChunkInstaller, Warning, "Chunk %i couldn't be mounted.", UnrealChunkID);
			InstallDelegate.Broadcast(UnrealChunkID, false);
		}
	};

	if (IsInGameThread())
	{
		BroadcastDelegateFunc();
	}
	else if (FTaskGraphInterface::IsRunning())
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, BroadcastDelegateFunc);
	}

	return bChunkMounted;
}

bool FGDKPlatformChunkInstall::UnMountPaks(uint32 UnrealChunkID)
{
	UE_LOGF(LogChunkInstaller, Log, "Unmounting pak files from chunk %i.", UnrealChunkID);

	// find and unmount all pak files
	const TArray<FString> PakchunkFiles = IGDKPackageManifestModule::Get().GetPakFilesInChunk(UnrealChunkID);
	bool bChunkUnmounted = true;
	for (const FString& PakchunkFile : PakchunkFiles)
	{
		FString PakLocation = TEXT("../../..") / PakchunkFile;
		PakLocation.ReplaceCharInline(TEXT('\\'), TEXT('/'));
		bool bPakFileUnmounted = FCoreDelegates::OnUnmountPak.IsBound() ? FCoreDelegates::OnUnmountPak.Execute(PakLocation) : false;

		if (bPakFileUnmounted)
		{
			UE_LOGF(LogChunkInstaller, Log, "Successfully unmounted %ls", *PakchunkFile);
		}
		else
		{
			UE_LOGF(LogChunkInstaller, Log, "Failed to unmount %ls", *PakchunkFile);
		}

		bChunkUnmounted &= bPakFileUnmounted;
	}

	if (!bChunkUnmounted)
	{
		UE_LOGF(LogChunkInstaller, Warning, "Streaming Install Chunk %i couldn't be unmounted.", UnrealChunkID);
	}

	return bChunkUnmounted;
}

void FGDKPlatformChunkInstall::ExternalNotifyChunkAvailable(uint32 PakchunkIndex)
{
	uint32 UnrealChunkID = IGDKPackageManifestModule::Get().GetChunkIDFromPakchunkIndex(PakchunkIndex);

	TFunction<void()> BroadcastDelegateFunc = [this, UnrealChunkID]()
	{
		InstallDelegate.Broadcast(UnrealChunkID, true);
	};

	if (IsInGameThread())
	{
		BroadcastDelegateFunc();
	}
	else if (FTaskGraphInterface::IsRunning())
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, BroadcastDelegateFunc);
	}
}

bool FGDKPlatformChunkInstall::PrioritizePakchunk(int32 PakchunkIndex, EChunkPriority::Type Priority)
{
	int32 UnrealChunkID = IGDKPackageManifestModule::Get().GetChunkIDFromPakchunkIndex(PakchunkIndex);
	return PrioritizeChunk(UnrealChunkID, Priority);
}

EChunkLocation::Type FGDKPlatformChunkInstall::GetPakchunkLocation(int32 PakchunkIndex)
{
	int32 UnrealChunkID = IGDKPackageManifestModule::Get().GetChunkIDFromPakchunkIndex(PakchunkIndex);
	return GetChunkLocation(UnrealChunkID);
}




#if WITH_CHUNKINSTALL_ASYNC_INIT
void FGDKPlatformChunkInstall::AsyncInit( TFunction<void(bool/*bSuccess*/)> OnInitComplete )
{
	UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, OnInitComplete = MoveTemp(OnInitComplete)]() mutable
	{
#if USE_GDK_PACKAGE_MANIFEST_INIT
		bool bSuccess = CreateChunkDataFromPackageManifest();
#else
		bool bSuccess = true;

		// create chunk data for the main package
		auto EnumeratePackageCallback = [](void* Context, const XPackageDetails* Details)
		{
			FGDKPlatformChunkInstall* This = (FGDKPlatformChunkInstall*)Context;
			This->CreateChunkDataForPackage(*Details);
			return true;
		};
		HRESULT hResult = XPackageEnumeratePackages( XPackageKind::Game, XPackageEnumerationScope::ThisOnly, this, EnumeratePackageCallback );
		UE_CLOGF(FAILED(hResult), LogChunkInstaller, Error, "XPackageEnumeratePackages failed: 0x%X", hResult);
		bSuccess = SUCCEEDED(hResult);
#endif // USE_GDK_PACKAGE_MANIFEST_INIT

		// signal completion on the gamethread
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, OnInitComplete = MoveTemp(OnInitComplete), bSuccess]() mutable
		{
			bIsInitialized = bSuccess;
			OnInitComplete(bSuccess);
		});
	});
}
#endif //WITH_CHUNKINSTALL_ASYNC_INIT

#if !USE_GDK_PACKAGE_MANIFEST_INIT
void FGDKPlatformChunkInstall::CreateChunkDataForPackage(const XPackageDetails& Details)
{
	UE_LOGF(LogChunkInstaller, Display, "Enumerating package %ls (%d.%d.%d.%d) %ls", UTF8_TO_TCHAR(Details.packageIdentifier), Details.version.major, Details.version.minor, Details.version.build, Details.version.revision, Details.installing ? TEXT(" - installing") : TEXT("") );
	FScopeLock Lock(&ChunkDataLock);

	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (startup only) XPackageEnumerateChunkAvailability / XPackageEnumerateFeatures / XPackageGetUserLocale are not safe to call on a time-sensitive thread

	// see if there are any Features
	bPackageHasFeatures = false;
	HRESULT hResult = XPackageEnumerateFeatures(PackageIdentifier, &bPackageHasFeatures, [](void* Context, const XPackageFeature* Feature)
	{
		// we've found a feature - set bPackageHasFeatures to true and stop enumerating
		*((bool*)Context) = true; 
		return false;
	});
	UE_CLOGF(FAILED(hResult), LogChunkInstaller, Error, "XPackageEnumerateFeatures failed: 0x%X", hResult);

	// enumerate all chunks
	hResult = XPackageEnumerateChunkAvailability( Details.packageIdentifier, XPackageChunkSelectorType::Chunk, this, [](void* Context, const XPackageChunkSelector* Selector, XPackageChunkAvailability Availability)
	{
		if ((Selector->chunkId & 0x40000000) == 0) //ignore system chunks
		{
			FGDKPlatformChunkInstall* This = (FGDKPlatformChunkInstall*)Context;
			This->ChunkMonitors.Emplace( ToUnrealChunkID(Selector->chunkId), new FChunkMonitor(*This, *Selector, Availability) );
		}
		return true;
	});
	UE_CLOGF(FAILED(hResult), LogChunkInstaller, Error, "XPackageEnumerateChunkAvailability (chunks) failed: 0x%X", hResult);

	// enumerate all Languages
	hResult = XPackageEnumerateChunkAvailability( Details.packageIdentifier, XPackageChunkSelectorType::Language, this, [](void* Context, const XPackageChunkSelector* Selector, XPackageChunkAvailability Availability)
	{
		FGDKPlatformChunkInstall* This = (FGDKPlatformChunkInstall*)Context;
		
		FName NamedChunk(Selector->language);
		This->LanguageChunks.Emplace(NamedChunk);
		This->NamedChunkMonitors.Emplace(NamedChunk, new FChunkMonitor(*This, *Selector, Availability) );
		return true;
	});
	UE_CLOGF(FAILED(hResult), LogChunkInstaller, Error, "XPackageEnumerateChunkAvailability (languages) failed: 0x%X", hResult);


	if (bPackageHasFeatures)
	{
		// enumerate all Features
		hResult = XPackageEnumerateChunkAvailability( Details.packageIdentifier, XPackageChunkSelectorType::Feature, this, [](void* Context, const XPackageChunkSelector* Selector, XPackageChunkAvailability Availability)
		{
			FGDKPlatformChunkInstall* This = (FGDKPlatformChunkInstall*)Context;

			FName NamedChunk(Selector->feature);
			This->OnDemandChunks.Emplace(NamedChunk);
			This->NamedChunkMonitors.Emplace(NamedChunk, new FChunkMonitor(*This, *Selector, Availability) );
			return true;
		});
		UE_CLOGF(FAILED(hResult), LogChunkInstaller, Error, "XPackageEnumerateChunkAvailability (features) failed: 0x%X", hResult);
	}
	else
	{
		// this package does not use Features - enumerate raw Tags instead
		hResult = XPackageEnumerateChunkAvailability( Details.packageIdentifier, XPackageChunkSelectorType::Tag, this, [](void* Context, const XPackageChunkSelector* Selector, XPackageChunkAvailability Availability)
		{
			FGDKPlatformChunkInstall* This = (FGDKPlatformChunkInstall*)Context;

			FName NamedChunk(Selector->tag);
			This->OnDemandChunks.Emplace(NamedChunk);
			This->NamedChunkMonitors.Emplace(NamedChunk, new FChunkMonitor(*This, *Selector, Availability) );
			return true;
		});
		UE_CLOGF(FAILED(hResult), LogChunkInstaller, Error, "XPackageEnumerateChunkAvailability (tags) failed: 0x%X", hResult);
	}

	UE_LOGF(LogChunkInstaller, Display, "Finished enumerating package chunks:");
	for (const FName& NamedChunk : OnDemandChunks)
	{
		UE_LOGF(LogChunkInstaller, Display, "\tOnDemand: %ls", *NamedChunk.ToString());
	}
	for (const FName& NamedChunk : LanguageChunks)
	{
		UE_LOGF(LogChunkInstaller, Display, "\tLanguage: %ls", *NamedChunk.ToString());
	}


	// read package locale
	FString PackageLocale;
	{
		char PackageLocaleAnsi[LOCALE_NAME_MAX_LENGTH];
		hResult = XPackageGetUserLocale(LOCALE_NAME_MAX_LENGTH, PackageLocaleAnsi);
		if (SUCCEEDED(hResult))
		{
			PackageLocale = ANSI_TO_TCHAR(PackageLocaleAnsi);
		}
		else
		{
			UE_CLOGF(FAILED(hResult), LogChunkInstaller, Error, "XPackageGetUserLocale failed: 0x%X", hResult);
			PackageLocale = FPlatformMisc::GetDefaultLocale();
		}
		UE_LOGF(LogChunkInstaller, Display, "Using package locale: %ls", *PackageLocale);
	}

	// read stage id overrides from the target settings
	TMap<FName, FName> StageIdOverrides;
	{
		FString RawStageIdOverrides; // StageIdOverrides=(("en","en-US"), ... )
		GConfig->GetString(FPlatformProperties::GetRuntimeSettingsClassName(), TEXT("StageIdOverrides"), RawStageIdOverrides, GEngineIni);
		RawStageIdOverrides.ReplaceInline(TEXT("("), TEXT(""));
		RawStageIdOverrides.ReplaceInline(TEXT(")"), TEXT(""));
		RawStageIdOverrides.ReplaceInline(TEXT("\""), TEXT(""));
		RawStageIdOverrides.TrimStartAndEndInline();
		if (!RawStageIdOverrides.IsEmpty())
		{
			TArray<FString> StageIdPairs;
			RawStageIdOverrides.ParseIntoArray( StageIdPairs, TEXT(","), false );
			check( (StageIdPairs.Num() % 2) == 0 ); // should have an even number of these as they're keyval pairs
			for (int32 Index = 0; Index < StageIdPairs.Num(); Index += 2)
			{
				const FString& StageId   = StageIdPairs[Index];
				const FString& CultureId = StageIdPairs[Index+1];
				if (!CultureId.IsEmpty())
				{
					StageIdOverrides.Add(FName(*CultureId), FName(*StageId));
				}
			}
		}
	}


	const TArray<FGDKPackageManifestChunk>& AllChunks = IGDKPackageManifestModule::Get().GetChunks();
	UE_LOGF(LogChunkInstaller, Display, "%d chunks in manifest", AllChunks.Num());

	for (const FGDKPackageManifestChunk& Chunk : AllChunks)
	{
		// record any Initial chunks, so that the install state can be queried (mostly for bundle support)
		// if using features, it will be necessary to manually add the Tag to the launch chunks in the ini file as the field is disabled in the editor for launch chunks
		if (Chunk.bIsInitial)
		{
			FName NamedChunk(*Chunk.Tag);
			InitialNamedChunksMap.Add(NamedChunk, Chunk.UnrealChunkID);
			UE_LOGF(LogChunkInstaller, Display, "\tInitial chunk %ls -> %d", *NamedChunk.ToString(), Chunk.UnrealChunkID);
		}

		// cache all named chunks that are not available in the current locale (for use by IsNamedChunkForCurrentLocale)
		if (!Chunk.Languages.IsEmpty())
		{
			TArray<FString> ChunkLanguages;
			Chunk.Languages.ParseIntoArray(ChunkLanguages, TEXT(";"));
			for (const FString& Language : ChunkLanguages)
			{
				UE_LOGF(LogChunkInstaller, Display, "\tlanguage %ls for chunk %d", *Language, Chunk.UnrealChunkID);

				if (Language != PackageLocale)
				{
					FName NamedChunk(Language);
					SecondaryLanguageNamedChunks.Add(NamedChunk);
					if (StageIdOverrides.Contains(NamedChunk))
					{
						SecondaryLanguageNamedChunks.Add(StageIdOverrides[NamedChunk]); // this allows IsNamedChunkForCurrentLocale to also work with UE Stage Ids in addition to GDK Culture Ids
					}
				}
			}
		}
	}

	UE_LOGF(LogChunkInstaller, Display, "Initialize complete. OnDemand: %d, Language: %d, Monitors: %d, NamedMonitors: %d", OnDemandChunks.Num(), LanguageChunks.Num(), ChunkMonitors.Num(), NamedChunkMonitors.Num());
}
#endif //!USE_GDK_PACKAGE_MANIFEST_INIT

#if USE_GDK_PACKAGE_MANIFEST_INIT
bool FGDKPlatformChunkInstall::CreateChunkDataFromPackageManifest()
{
	bool bSuccess = true;

	const TArray<FGDKPackageManifestChunk>& ManifestChunks = IGDKPackageManifestModule::Get().GetChunks();
	const TArray<FGDKPackageManifestFeature>& ManifestFeatures = IGDKPackageManifestModule::Get().GetFeatures();
	const TArray<FName>& ManifestLanguages = IGDKPackageManifestModule::Get().GetLanguages();
	const TArray<FName>& ManifestTags = IGDKPackageManifestModule::Get().GetTags();

	UE_LOGF(LogChunkInstaller, Display, "Enumerating package %ls", UTF8_TO_TCHAR(PackageIdentifier) );
	FScopeLock Lock(&ChunkDataLock);

	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (startup only) XPackageFindChunkAvailability / XPackageGetUserLocale are not safe to call on a time-sensitive thread

	// see if there are any Features
	bPackageHasFeatures = ManifestFeatures.Num() > 0;

	// enumerate all Chunks
	for (const FGDKPackageManifestChunk& ManifestChunk : ManifestChunks)
	{
		XPackageChunkSelector ChunkSelector;
		ChunkSelector.type = XPackageChunkSelectorType::Chunk;
		ChunkSelector.chunkId = FromUnrealChunkID(ManifestChunk.UnrealChunkID);

		XPackageChunkAvailability ChunkAvailability = XPackageChunkAvailability::Unavailable;
		HRESULT hResult = XPackageFindChunkAvailability(PackageIdentifier, 1, &ChunkSelector, &ChunkAvailability);
		if (FAILED(hResult))
		{
			bSuccess = false;
			UE_LOGF(LogChunkInstaller, Error, "XPackageFindChunkAvailability (chunk %d) failed: 0x%X", ManifestChunk.UnrealChunkID, hResult);
			continue;
		}

		ChunkMonitors.Emplace( ManifestChunk.UnrealChunkID, new FChunkMonitor(*this, ChunkSelector, ChunkAvailability) );
	}

	// enumerate all Languages
	for (const FName& ManifestLanguage : ManifestLanguages)
	{
		FTCHARToUTF8 UTF8Language(*ManifestLanguage.ToString());

		XPackageChunkSelector ChunkSelector;
		ChunkSelector.type = XPackageChunkSelectorType::Language;
		ChunkSelector.language = UTF8Language.Get();

		XPackageChunkAvailability ChunkAvailability = XPackageChunkAvailability::Unavailable;
		HRESULT hResult = XPackageFindChunkAvailability(PackageIdentifier, 1, &ChunkSelector, &ChunkAvailability);
		if (FAILED(hResult))
		{
			bSuccess = false;
			UE_LOGF(LogChunkInstaller, Error, "XPackageFindChunkAvailability (language %ls) failed: 0x%X", *ManifestLanguage.ToString(), hResult);
			continue;
		}

		LanguageChunks.Emplace(ManifestLanguage);
		NamedChunkMonitors.Emplace(ManifestLanguage, new FChunkMonitor(*this, ChunkSelector, ChunkAvailability) );
	}


	if (bPackageHasFeatures)
	{
		// enumerate all Features
		for (const FGDKPackageManifestFeature& ManifestFeature : ManifestFeatures)
		{
			FTCHARToUTF8 UTF8Feature(*ManifestFeature.Id);

			XPackageChunkSelector ChunkSelector;
			ChunkSelector.type = XPackageChunkSelectorType::Feature;
			ChunkSelector.feature = UTF8Feature.Get();

			XPackageChunkAvailability ChunkAvailability = XPackageChunkAvailability::Unavailable;
			HRESULT hResult = XPackageFindChunkAvailability(PackageIdentifier, 1, &ChunkSelector, &ChunkAvailability);
			if (FAILED(hResult))
			{
				bSuccess = false;
				UE_LOGF(LogChunkInstaller, Error, "XPackageFindChunkAvailability (feature %ls) failed: 0x%X", *ManifestFeature.Id, hResult);
				continue;
			}

			FName NamedChunk(ManifestFeature.Id);
			OnDemandChunks.Emplace(NamedChunk);
			NamedChunkMonitors.Emplace(NamedChunk, new FChunkMonitor(*this, ChunkSelector, ChunkAvailability) );
		}
	}
	else
	{
		// this package does not use Features - enumerate raw Tags instead
		for (const FName& ManifestTag : ManifestTags)
		{
			FTCHARToUTF8 UTF8Tag(*ManifestTag.ToString());

			XPackageChunkSelector ChunkSelector;
			ChunkSelector.type = XPackageChunkSelectorType::Tag;
			ChunkSelector.tag = UTF8Tag.Get();

			XPackageChunkAvailability ChunkAvailability = XPackageChunkAvailability::Unavailable;
			HRESULT hResult = XPackageFindChunkAvailability(PackageIdentifier, 1, &ChunkSelector, &ChunkAvailability);
			if (FAILED(hResult))
			{
				bSuccess = false;
				UE_LOGF(LogChunkInstaller, Error, "XPackageFindChunkAvailability (tag %ls) failed: 0x%X", *ManifestTag.ToString(), hResult);
				continue;
			}

			OnDemandChunks.Emplace(ManifestTag);
			NamedChunkMonitors.Emplace(ManifestTag, new FChunkMonitor(*this, ChunkSelector, ChunkAvailability) );
		}
	}

	UE_LOGF(LogChunkInstaller, Display, "Finished enumerating package chunks:");
	for (const FName& NamedChunk : OnDemandChunks)
	{
		UE_LOGF(LogChunkInstaller, Display, "\tOnDemand: %ls", *NamedChunk.ToString());
	}
	for (const FName& NamedChunk : LanguageChunks)
	{
		UE_LOGF(LogChunkInstaller, Display, "\tLanguage: %ls", *NamedChunk.ToString());
	}


	// read package locale
	FString PackageLocale;
	{
		char PackageLocaleAnsi[LOCALE_NAME_MAX_LENGTH];
		HRESULT hResult = XPackageGetUserLocale(LOCALE_NAME_MAX_LENGTH, PackageLocaleAnsi);
		if (SUCCEEDED(hResult))
		{
			PackageLocale = ANSI_TO_TCHAR(PackageLocaleAnsi);
		}
		else
		{
			bSuccess = false;
			UE_CLOGF(FAILED(hResult), LogChunkInstaller, Error, "XPackageGetUserLocale failed: 0x%X", hResult);
			PackageLocale = FPlatformMisc::GetDefaultLocale();
		}
		UE_LOGF(LogChunkInstaller, Display, "Using package locale: %ls", *PackageLocale);
	}

	// read stage id overrides from the target settings
	TMap<FName, FName> StageIdOverrides;
	{
		FString RawStageIdOverrides; // StageIdOverrides=(("en","en-US"), ... )
		GConfig->GetString(FPlatformProperties::GetRuntimeSettingsClassName(), TEXT("StageIdOverrides"), RawStageIdOverrides, GEngineIni);
		RawStageIdOverrides.ReplaceInline(TEXT("("), TEXT(""));
		RawStageIdOverrides.ReplaceInline(TEXT(")"), TEXT(""));
		RawStageIdOverrides.ReplaceInline(TEXT("\""), TEXT(""));
		RawStageIdOverrides.TrimStartAndEndInline();
		if (!RawStageIdOverrides.IsEmpty())
		{
			TArray<FString> StageIdPairs;
			RawStageIdOverrides.ParseIntoArray( StageIdPairs, TEXT(","), false );
			check( (StageIdPairs.Num() % 2) == 0 ); // should have an even number of these as they're keyval pairs
			for (int32 Index = 0; Index < StageIdPairs.Num(); Index += 2)
			{
				const FString& StageId   = StageIdPairs[Index];
				const FString& CultureId = StageIdPairs[Index+1];
				if (!CultureId.IsEmpty())
				{
					StageIdOverrides.Add(FName(*CultureId), FName(*StageId));
				}
			}
		}
	}


	UE_LOGF(LogChunkInstaller, Display, "%d chunks in manifest", ManifestChunks.Num());

	for (const FGDKPackageManifestChunk& Chunk : ManifestChunks)
	{
		// record any Initial chunks, so that the install state can be queried (mostly for bundle support)
		// if using features, it will be necessary to manually add the Tag to the launch chunks in the ini file as the field is disabled in the editor for launch chunks
		if (Chunk.bIsInitial)
		{
			FName NamedChunk(*Chunk.Tag);
			InitialNamedChunksMap.Add(NamedChunk, Chunk.UnrealChunkID);
			UE_LOGF(LogChunkInstaller, Display, "\tInitial chunk %ls -> %d", *NamedChunk.ToString(), Chunk.UnrealChunkID);
		}

		// cache all named chunks that are not available in the current locale (for use by IsNamedChunkForCurrentLocale)
		if (!Chunk.Languages.IsEmpty())
		{
			TArray<FString> ChunkLanguages;
			Chunk.Languages.ParseIntoArray(ChunkLanguages, TEXT(";"));
			for (const FString& Language : ChunkLanguages)
			{
				UE_LOGF(LogChunkInstaller, Display, "\tlanguage %ls for chunk %d", *Language, Chunk.UnrealChunkID);

				if (Language != PackageLocale)
				{
					FName NamedChunk(Language);
					SecondaryLanguageNamedChunks.Add(NamedChunk);
					if (StageIdOverrides.Contains(NamedChunk))
					{
						SecondaryLanguageNamedChunks.Add(StageIdOverrides[NamedChunk]); // this allows IsNamedChunkForCurrentLocale to also work with UE Stage Ids in addition to GDK Culture Ids
					}
				}
			}
		}
	}

	UE_LOGF(LogChunkInstaller, Display, "Initialize complete. OnDemand: %d, Language: %d, Monitors: %d, NamedMonitors: %d", OnDemandChunks.Num(), LanguageChunks.Num(), ChunkMonitors.Num(), NamedChunkMonitors.Num());
	UE_CLOG( !bSuccess, LogChunkInstaller, Fatal, TEXT("Encountered errors when initializing the chunk installer. Unable to continue."));
	return bSuccess;
}
#endif //USE_GDK_PACKAGE_MANIFEST_INIT



FGDKPlatformChunkInstall::FChunkMonitor* FGDKPlatformChunkInstall::FindChunkMonitor( uint32 UnrealChunkID )
{
#if WITH_CHUNKINSTALL_ASYNC_INIT
	checkf(bIsInitialized, TEXT("wait for AsyncInit first"));
#endif

	TUniquePtr<FChunkMonitor>* ResultPtr = ChunkMonitors.Find(UnrealChunkID);
	return ResultPtr ? ResultPtr->Get() : nullptr;
}

FGDKPlatformChunkInstall::FChunkMonitor* FGDKPlatformChunkInstall::FindChunkMonitor( const FName NamedChunk )
{
#if WITH_CHUNKINSTALL_ASYNC_INIT
	checkf(bIsInitialized, TEXT("wait for AsyncInit first"));
#endif

	TUniquePtr<FChunkMonitor>* ResultPtr = NamedChunkMonitors.Find(NamedChunk);
	return ResultPtr ? ResultPtr->Get() : nullptr;
}

void FGDKPlatformChunkInstall::OnChunkInstallationChanged( uint32 UnrealChunkID, bool bIsInstalled )
{
	UE_LOGF(LogChunkInstaller, Log, "Chunk %d %ls", UnrealChunkID, bIsInstalled ? TEXT("installed") : TEXT("uninstalled") );

	if (bAutoMountPaks)
	{
		if (bIsInstalled)
		{
			MountPaks(UnrealChunkID);
		}
		else
		{
			UnMountPaks(UnrealChunkID);
		}
	}
}

void FGDKPlatformChunkInstall::OnNamedChunkInstallationChanged( const FName NamedChunk, bool bIsInstalled )
{
	if (!bIsInstalled)
	{
		//nb. we are inside a FScopeLock Lock(&ChunkDataLock) from the caller, OnProgressChanged

		if (CurrentUninstallingChunks.Contains(NamedChunk))
		{
			CurrentUninstallingChunks.Remove(NamedChunk);
		}
		else if (bPackageHasFeatures && GGDKChunkInstallTerminateGameOnExternalFeatureRemoval)
		{
			UE_LOGF(LogChunkInstaller, Log, "Named chunk %ls removed externally - terminating the game", *NamedChunk.ToString());	
			RequestEngineExit(FString::Printf(TEXT("Named chunk %s removed externally"), *NamedChunk.ToString()));
		}
	}


	UE_LOGF(LogChunkInstaller, Log, "Named chunk %ls %ls", *NamedChunk.ToString(), bIsInstalled ? TEXT("installed") : TEXT("uninstalled") );

	EChunkLocation::Type Location = (bIsInstalled ? EChunkLocation::LocalFast : EChunkLocation::NotAvailable);
	if (IsInGameThread())
	{
		DoNamedChunkCompleteCallbacks(NamedChunk, Location, true);
	}
	else if (FTaskGraphInterface::IsRunning())
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, NamedChunk, Location]
		{
			DoNamedChunkCompleteCallbacks(NamedChunk, Location, true);
		});
	}
}





EChunkLocation::Type FGDKPlatformChunkInstall::GetChunkLocation(uint32 UnrealChunkID)
{
	const FChunkMonitor* ChunkMonitor = FindChunkMonitor(UnrealChunkID);
	if (ChunkMonitor != nullptr)
	{
		return ChunkMonitor->GetLocation();
	}

	return EChunkLocation::DoesNotExist;
}

EChunkLocation::Type FGDKPlatformChunkInstall::GetNamedChunkLocation(const FName NamedChunk)
{
	if (InitialNamedChunksMap.Contains(NamedChunk))
	{
		return EChunkLocation::LocalFast;
	}

	const FChunkMonitor* ChunkMonitor = FindChunkMonitor(NamedChunk);
	if (ChunkMonitor != nullptr)
	{
		return ChunkMonitor->GetLocation();
	}

	return EChunkLocation::DoesNotExist;
}


float FGDKPlatformChunkInstall::GetChunkProgress( uint32 PakchunkIndex, EChunkProgressReportingType::Type ReportType )
{
	uint32 UnrealChunkID = IGDKPackageManifestModule::Get().GetChunkIDFromPakchunkIndex(PakchunkIndex);
	const FChunkMonitor* ChunkMonitor = FindChunkMonitor(UnrealChunkID);
	if (ChunkMonitor != nullptr)
	{
		return ChunkMonitor->GetProgress(ReportType);
	}

	return 0.0f;
}

float FGDKPlatformChunkInstall::GetNamedChunkProgress(const FName NamedChunk, EChunkProgressReportingType::Type ReportType)
{
	if (InitialNamedChunksMap.Contains(NamedChunk))
	{
		return (ReportType == EChunkProgressReportingType::PercentageComplete) ? 100.0f : 0.0f;
	}

	const FChunkMonitor* ChunkMonitor = FindChunkMonitor(NamedChunk);
	if (ChunkMonitor != nullptr)
	{
		return ChunkMonitor->GetProgress(ReportType);
	}

	return 0.0f;
}




bool FGDKPlatformChunkInstall::PrioritizeChunk( uint32 UnrealChunkID, EChunkPriority::Type Priority )
{
	FChunkMonitor* ChunkMonitor = FindChunkMonitor(UnrealChunkID);
	if (ChunkMonitor != nullptr)
	{
		return ChunkMonitor->SetPriority(Priority);
	}

	return false;
}

bool FGDKPlatformChunkInstall::PrioritizeNamedChunk(const FName NamedChunk, EChunkPriority::Type Priority)
{
	FChunkMonitor* ChunkMonitor = FindChunkMonitor(NamedChunk);
	if (ChunkMonitor != nullptr)
	{
		return ChunkMonitor->SetPriority(Priority);
	}

	return false;
}


bool FGDKPlatformChunkInstall::CancelNamedChunksInstall(const TArrayView<const FName>& NamedChunks)
{
#if WITH_CHUNKINSTALL_ASYNC_INIT
	checkf(bIsInitialized, TEXT("wait for AsyncInit first"));
#endif

	// early out
	if (!bIsPackaged || NamedChunks.Num() == 0)
	{
		return false;
	}

	if (!GGDKChunkInstallEnableCancellation)
	{
		UE_LOGF(LogChunkInstaller, Log, "CancelNamedChunksInstall: cancellation is disabled");
		return false;
	}

	// Only cancel chunks that are not yet installed because the content would be uninstalled
	TArray<FName> ChunksToCancel;
	for (FName NamedChunk : NamedChunks)
	{
		const FChunkMonitor* ChunkMonitor = FindChunkMonitor(NamedChunk);
		if (ChunkMonitor == nullptr)
		{
			UE_LOGF(LogChunkInstaller, Error, "cannot cancel unknown named chunk: %ls", *NamedChunk.ToString());
		}
		else if (ChunkMonitor->bInstalled)
		{
			UE_LOGF(LogChunkInstaller, Log, "skipping cancel of already installed named chunk: %ls", *NamedChunk.ToString());
		}
		else
		{
			UE_LOGF(LogChunkInstaller, Log, "cancelling named chunk install: %ls", *NamedChunk.ToString());
			ChunksToCancel.Add(NamedChunk);
		}
	}

	return ChunksToCancel.Num() > 0 && UninstallNamedChunks(MakeArrayView(ChunksToCancel));
}


bool FGDKPlatformChunkInstall::IsNamedChunkInProgress(const FName NamedChunk)
{
	FChunkMonitor* ChunkMonitor = FindChunkMonitor(NamedChunk);
	if (ChunkMonitor == nullptr)
	{
		return false;
	}

	GDK_SCOPE_NOT_TIME_SENSITIVE(); // XPackageFindChunkAvailability is not safe to call on a time-sensitive thread

	// check the availability of the given chunk
	XPackageChunkAvailability InstallationState;
	HRESULT hResult = XPackageFindChunkAvailability( PackageIdentifier, 1, &ChunkMonitor->Selector, &InstallationState );
	UE_CLOGF(FAILED(hResult), LogChunkInstaller, Error, "Failed to query named chunk availability for %ls - 0x%X", *NamedChunk.ToString(), hResult);

	return SUCCEEDED(hResult) && InstallationState == XPackageChunkAvailability::Pending;
}


bool FGDKPlatformChunkInstall::Tick( float DeltaSeconds )
{
	// Active progress polling: Poll XPackageGetInstallationProgress for all chunk monitors
	// that are not yet installed. This ensures progress is updated even if the
	// XPackageRegisterInstallationProgressChanged callbacks are not firing properly.
	{
		FScopeLock Lock(&ChunkDataLock);
		for (auto& Pair : NamedChunkMonitors)
		{
			FChunkMonitor* ChunkMonitor = Pair.Value.Get();
			if (ChunkMonitor && !ChunkMonitor->bInstalled && ChunkMonitor->InstallationMonitorHandle != nullptr)
			{
				XPackageInstallationProgress Progress;
				XPackageGetInstallationProgress(ChunkMonitor->InstallationMonitorHandle, &Progress);

				uint8 NewProgress = (Progress.totalBytes > 0) ? static_cast<uint8>((Progress.installedBytes * 100) / Progress.totalBytes) : 0;

				// Only update and log if progress changed
				if (NewProgress != ChunkMonitor->CachedProgress)
				{
					ChunkMonitor->CachedProgress = NewProgress;
					UE_LOGF(LogChunkInstaller, Verbose, "%ls installation progress polled: %llu / %llu bytes (%d%%).%ls",
						*ChunkMonitor->GetName(), Progress.installedBytes, Progress.totalBytes, NewProgress, Progress.completed ? TEXT(" completed") : TEXT(""));
				}

				// Check for completion - must notify via OnNamedChunkInstallationChanged just like OnProgressChange does
				if (Progress.completed && !ChunkMonitor->bInstalled && Progress.totalBytes > 0)
				{
					ChunkMonitor->bInstalled = true;
					ChunkMonitor->bWasInProgress = false;
					UE_LOGF(LogChunkInstaller, Display, "%ls installation completed via polling", *ChunkMonitor->GetName());
					// Notify dependent systems - Pair.Key is the FName for named chunks
					OnNamedChunkInstallationChanged(Pair.Key, true);
				}
			}
		}

		// Also check indexed chunk monitors
		for (auto& Pair : ChunkMonitors)
		{
			FChunkMonitor* ChunkMonitor = Pair.Value.Get();
			if (ChunkMonitor && !ChunkMonitor->bInstalled && ChunkMonitor->InstallationMonitorHandle != nullptr)
			{
				XPackageInstallationProgress Progress;
				XPackageGetInstallationProgress(ChunkMonitor->InstallationMonitorHandle, &Progress);

				uint8 NewProgress = (Progress.totalBytes > 0) ? static_cast<uint8>((Progress.installedBytes * 100) / Progress.totalBytes) : 0;

				// Only update and log if progress changed
				if (NewProgress != ChunkMonitor->CachedProgress)
				{
					ChunkMonitor->CachedProgress = NewProgress;
					UE_LOGF(LogChunkInstaller, Verbose, "%ls installation progress polled: %llu / %llu bytes (%d%%).%ls",
						*ChunkMonitor->GetName(), Progress.installedBytes, Progress.totalBytes, NewProgress, Progress.completed ? TEXT(" completed") : TEXT(""));
				}

				// Check for completion - must notify via OnChunkInstallationChanged just like OnProgressChange does
				if (Progress.completed && !ChunkMonitor->bInstalled && Progress.totalBytes > 0)
				{
					ChunkMonitor->bInstalled = true;
					ChunkMonitor->bWasInProgress = false;
					UE_LOGF(LogChunkInstaller, Display, "%ls installation completed via polling", *ChunkMonitor->GetName());
					// Notify dependent systems - Pair.Key is the UnrealChunkID for indexed chunks
					OnChunkInstallationChanged(Pair.Key, true);
				}
			}
		}
	}

	if (PendingInstallChunks.Num() > 0)
	{
		// async callback
		auto AsyncInstallComplete = []( XAsyncBlock* AsyncBlock)
		{
			// collect result
			XPackageInstallationMonitorHandle Monitor;
			HRESULT hResult = XPackageInstallChunksResult(AsyncBlock, &Monitor);
			delete AsyncBlock;
			if (FAILED(hResult))
			{
				UE_LOGF(LogChunkInstaller, Error, "Async named chunk install failed to start - 0x%X", hResult);
			}
			else
			{
				// FChunkMonitor takes care of the actual tracking so we can just close the handle here
				XPackageCloseInstallationMonitorHandle(Monitor); 
			}
		};

		// only request chunks that can be installed... otherwise signal the callback immediately
		TSet<FName> ChunksToInstall;
		for (FName NamedChunk : PendingInstallChunks)
		{
			const FChunkMonitor* ChunkMonitor = FindChunkMonitor(NamedChunk);
			if (ensure(ChunkMonitor) && ChunkMonitor->bInstalled)
			{		
				UE_LOGF(LogChunkInstaller, Log, "Named chunk %ls is already installed", *NamedChunk.ToString() );
				DoNamedChunkCompleteCallbacks(NamedChunk, EChunkLocation::LocalFast, true);
			}
			else if (UnavailableChunks.Contains(NamedChunk))
			{
				UE_LOGF(LogChunkInstaller, Log, "Named chunk %ls is not available in the current locale/device", *NamedChunk.ToString() );
				DoNamedChunkCompleteCallbacks(NamedChunk, EChunkLocation::DoesNotExist, false);
			}
			else
			{
				ChunksToInstall.Add(NamedChunk);
			}
		}


		if (ChunksToInstall.Num() > 0)
		{
			// prepare the async chunk install
			XAsyncBlock* AsyncBlock = new XAsyncBlock{};
			AsyncBlock->queue = FGDKAsyncTaskQueue::GetGenericQueue();
			AsyncBlock->callback = AsyncInstallComplete;

			// install the chunks
			FGDKCustomChunkSelector Chunks(ChunksToInstall, this);
			HRESULT hResult = XPackageInstallChunksAsync( PackageIdentifier, Chunks.Num(), Chunks.Get(), GDK_CHUNKINSTALL_UPDATEINTERVAL_MS, (GGDKChunkInstallSuppressUserConfirmation != 0), AsyncBlock );
			if (FAILED(hResult))
			{
				delete AsyncBlock;
				UE_LOGF(LogChunkInstaller, Error, "Failed to start async installing named chunks %ls - 0x%X", *Chunks.ToString(), hResult);
				for (FName NamedChunk : ChunksToInstall)
				{
					DoNamedChunkCompleteCallbacks(NamedChunk, EChunkLocation::DoesNotExist, false);
				}

			}
		}

		PendingInstallChunks.Reset();
	}

	if (PendingUninstallChunks.Num() > 0)
	{
		// only request chunks that can be uninstalled... otherwise signal the callback immediately
		TSet<FName> ChunksToUninstall;
		for (FName NamedChunk : PendingUninstallChunks)
		{
			const FChunkMonitor* ChunkMonitor = FindChunkMonitor(NamedChunk);
			if (ensure(ChunkMonitor) && !ChunkMonitor->bInstalled && !IsNamedChunkInProgress(NamedChunk))
			{
				UE_LOGF(LogChunkInstaller, Log, "Named chunk %ls is already uninstalled", *NamedChunk.ToString() );
				DoNamedChunkCompleteCallbacks(NamedChunk, EChunkLocation::NotAvailable, true);
			}
			else if (UnavailableChunks.Contains(NamedChunk))
			{
				UE_LOGF(LogChunkInstaller, Log, "Named chunk %ls is not available in the current locale/device", *NamedChunk.ToString() );
				DoNamedChunkCompleteCallbacks(NamedChunk, EChunkLocation::DoesNotExist, false);
			}
			else
			{
				UE_CLOGF(IsNamedChunkInProgress(NamedChunk), LogChunkInstaller, Log, "Named chunk %ls in progress and will be cancelled", *NamedChunk.ToString() );
				ChunksToUninstall.Add(NamedChunk);
			}
		}


		if (ChunksToUninstall.Num() > 0)
		{
			GDK_SCOPE_NOT_TIME_SENSITIVE(); // XPackageUninstallChunks is not safe to call on a time-sensitive thread

			// uninstall the chunks
			FGDKCustomChunkSelector Chunks(ChunksToUninstall, this);
			HRESULT hResult = XPackageUninstallChunks( PackageIdentifier, Chunks.Num(), Chunks.Get() );
			if (FAILED(hResult))
			{
				UE_LOGF(LogChunkInstaller, Error, "Failed to start uninstalling named chunks %ls - 0x%X", *Chunks.ToString(), hResult);
				for (FName NamedChunk : ChunksToUninstall)
				{
					DoNamedChunkCompleteCallbacks(NamedChunk, EChunkLocation::LocalFast, false);
				}
			}
			else
			{
				FScopeLock Lock(&ChunkDataLock);
				CurrentUninstallingChunks = CurrentUninstallingChunks.Union(ChunksToUninstall);
			}
		}
		PendingUninstallChunks.Reset();
	}

	// stop ticking
	RemoveFromTicker();
	return false;
}


bool FGDKPlatformChunkInstall::InstallNamedChunks(const TArrayView<const FName>& NamedChunks)
{
#if WITH_CHUNKINSTALL_ASYNC_INIT
	checkf(bIsInitialized, TEXT("wait for AsyncInit first"));
#endif

	// early out
	if (!bIsPackaged || NamedChunks.Num() == 0)
	{
		DoNamedChunkCompleteCallbacks(NamedChunks, EChunkLocation::DoesNotExist, false);
		return false;
	}

	// validate requested chunks
	TSet<FName> RequestedChunks;
	for (FName NamedChunk : NamedChunks)
	{
		if (InitialNamedChunksMap.Contains(NamedChunk))
		{
			UE_LOGF(LogChunkInstaller, Error, "launch chunk always installed: %ls", *NamedChunk.ToString());
			DoNamedChunkCompleteCallbacks(NamedChunk, EChunkLocation::LocalFast, true);
		}
		else if (NamedChunkMonitors.Contains(NamedChunk))
		{
			UE_LOGF(LogChunkInstaller, Error, "installing named chunk: %ls", *NamedChunk.ToString());
			RequestedChunks.Add(NamedChunk);
		}
		else
		{
			UE_LOGF(LogChunkInstaller, Error, "cannot install unknown named chunk: %ls", *NamedChunk.ToString());
			DoNamedChunkCompleteCallbacks(NamedChunk, EChunkLocation::DoesNotExist, false);
		}
	}

	// update pending chunk sets
	PendingUninstallChunks = PendingUninstallChunks.Difference(RequestedChunks); // remove chunks from uninstall request
	PendingInstallChunks = PendingInstallChunks.Union(RequestedChunks); // add chunks to install request

	// request a tick if there's anything to do
	if (PendingUninstallChunks.Num() > 0 || PendingInstallChunks.Num() > 0)
	{
		AddToTicker();
		return true;
	}

	return false;

}



bool FGDKPlatformChunkInstall::UninstallNamedChunks(const TArrayView<const FName>& NamedChunks)
{
#if WITH_CHUNKINSTALL_ASYNC_INIT
	checkf(bIsInitialized, TEXT("wait for AsyncInit first"));
#endif

	// early out
	if (!bIsPackaged || NamedChunks.Num() == 0)
	{
		DoNamedChunkCompleteCallbacks(NamedChunks, EChunkLocation::DoesNotExist, false);
		return false;
	}

	// validate requested chunks
	TSet<FName> RequestedChunks;
	for (FName NamedChunk : NamedChunks)
	{
		if (InitialNamedChunksMap.Contains(NamedChunk))
		{
			UE_LOGF(LogChunkInstaller, Error, "cannot remove launch chunk: %ls", *NamedChunk.ToString());
			DoNamedChunkCompleteCallbacks(NamedChunk, EChunkLocation::LocalFast, false);
		}
		else if (NamedChunkMonitors.Contains(NamedChunk))
		{
			UE_LOGF(LogChunkInstaller, Error, "uninstalling named chunk: %ls", *NamedChunk.ToString());
			RequestedChunks.Add(NamedChunk);
		}
		else
		{
			UE_LOGF(LogChunkInstaller, Error, "cannot uninstall unknown named chunk: %ls", *NamedChunk.ToString());
			DoNamedChunkCompleteCallbacks(NamedChunk, EChunkLocation::DoesNotExist, false);
		}
	}

	// update pending chunk sets
	PendingUninstallChunks = PendingUninstallChunks.Union(RequestedChunks); // add chunks to uninstall request
	PendingInstallChunks = PendingInstallChunks.Difference(RequestedChunks); // remove chunks from install request

	// request a tick if there's anything to do
	if (PendingUninstallChunks.Num() > 0 || PendingInstallChunks.Num() > 0)
	{
		AddToTicker();
		return true;
	}

	return false;

}



TArray<FName> FGDKPlatformChunkInstall::GetNamedChunksByType(ENamedChunkType NamedChunkType) const
{
#if WITH_CHUNKINSTALL_ASYNC_INIT
	checkf(bIsInitialized, TEXT("wait for AsyncInit first"));
#endif

	switch(NamedChunkType)
	{
		case ENamedChunkType::OnDemand:	return OnDemandChunks;
		case ENamedChunkType::Language:	return LanguageChunks;
	}
	checkNoEntry();
	return TArray<FName>();
}

ENamedChunkType FGDKPlatformChunkInstall::GetNamedChunkType(const FName NamedChunk) const
{
#if WITH_CHUNKINSTALL_ASYNC_INIT
	checkf(bIsInitialized, TEXT("wait for AsyncInit first"));
#endif

	if (OnDemandChunks.Contains(NamedChunk))
	{
		return ENamedChunkType::OnDemand;
	}
	else if (LanguageChunks.Contains(NamedChunk))
	{
		return ENamedChunkType::Language;
	}
	else
	{
		return ENamedChunkType::Invalid;
	}
}



// NOTE: this will return all *potential* files for the given named chunk (e.g. tag # (AND) operator is not considered for Features & Recipes)
bool FGDKPlatformChunkInstall::GetPakFilesInNamedChunk( const FName NamedChunk, TArray<FString>& OutFilesInChunk) const
{
#if WITH_CHUNKINSTALL_ASYNC_INIT
	checkf(bIsInitialized, TEXT("wait for AsyncInit first"));
#endif

	OutFilesInChunk.Reset();

	if (bPackageHasFeatures)
	{
		// get all chunks that comprise this feature
		TArray<int32> UnrealChunkIDs;
		if (!IGDKPackageManifestModule::Get().GetUnrealChunkIDsByFeature(NamedChunk.ToString(), UnrealChunkIDs))
		{
			return false;
		}

		// collect all the files from these chunks
		for (int32 UnrealChunkID : UnrealChunkIDs)
		{
			OutFilesInChunk.Append( IGDKPackageManifestModule::Get().GetPakFilesInChunk(UnrealChunkID) );
		}

		return true;
	}
	else
	{
		// find the chunk with the given tag & collect the files
		const FString Tag = NamedChunk.ToString();
		const TArray<FGDKPackageManifestChunk>& AllChunks = IGDKPackageManifestModule::Get().GetChunks();
		for (const FGDKPackageManifestChunk& Chunk : AllChunks)
		{
			if (Chunk.Tag == Tag)
			{
				OutFilesInChunk.Append( IGDKPackageManifestModule::Get().GetPakFilesInChunk(Chunk.UnrealChunkID) );
				return true;
			}
		}

		return false;
	}

}

bool FGDKPlatformChunkInstall::GetNamedChunkInstallationStatus( const FName NamedChunk, FChunkInstallationStatusDetail& OutChunkStatusDetail ) const
{
	const FChunkMonitor* ChunkMonitor = FindChunkMonitor(NamedChunk);
	if (ChunkMonitor != nullptr)
	{
		XPackageInstallationProgress Progress;
		XPackageGetInstallationProgress( ChunkMonitor->InstallationMonitorHandle, &Progress );
		OutChunkStatusDetail.CurrentInstallSize = Progress.installedBytes;
		OutChunkStatusDetail.FullInstallSize = Progress.totalBytes;
		OutChunkStatusDetail.bIsInstalled = ChunkMonitor->bInstalled;

		return true;
	}

	// check initial chunks (these are always installed)
	TArray<int32> InitialChunkIDs;
	InitialNamedChunksMap.MultiFind(NamedChunk, InitialChunkIDs);
	if (InitialChunkIDs.Num() > 0 )
	{
		uint64 TotalBytes = 0;
		for (int32 InitialChunkID : InitialChunkIDs)
		{
			ChunkMonitor = FindChunkMonitor(InitialChunkID);
			if (ChunkMonitor != nullptr)
			{
				XPackageInstallationProgress Progress;
				XPackageGetInstallationProgress( ChunkMonitor->InstallationMonitorHandle, &Progress );
				TotalBytes += Progress.totalBytes;
			}
		}

		OutChunkStatusDetail.FullInstallSize = TotalBytes;
		OutChunkStatusDetail.CurrentInstallSize = TotalBytes;
		OutChunkStatusDetail.bIsInstalled = true;
		return true;
	}

	return false;
}


bool FGDKPlatformChunkInstall::IsNamedChunkForCurrentLocale(const FName NamedChunk) const
{
#if WITH_CHUNKINSTALL_ASYNC_INIT
	checkf(bIsInitialized, TEXT("wait for AsyncInit first"));
#endif

	return !SecondaryLanguageNamedChunks.Contains(NamedChunk);
}


bool FGDKPlatformChunkInstall::SetAutoPakMountingEnabled( bool bEnabled )
{
	bAutoMountPaks = bEnabled;
	return true;
}









void FGDKPlatformChunkInstall::AddToTicker()
{
	if (!TickHandle.IsValid())
	{
		FTSTicker& Ticker = FTSTicker::GetCoreTicker();

		// Register delegate for ticker callback
		FTickerDelegate TickDelegate = FTickerDelegate::CreateRaw(this, &FGDKPlatformChunkInstall::Tick);
		TickHandle = Ticker.AddTicker(TickDelegate, 0.1f);
	}
}

void FGDKPlatformChunkInstall::RemoveFromTicker()
{
	FTSTicker& Ticker = FTSTicker::GetCoreTicker();

	// Unregister ticker delegate
	if (TickHandle.IsValid())
	{
		Ticker.RemoveTicker(TickHandle);
		TickHandle.Reset();
	}
}





void FGDKPlatformChunkInstall::DebugDumpChunkState()
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (debug only) XPackageFindChunkAvailability is not safe to call on a time-sensitive thread

	FGDKPlatformChunkInstall* ChunkInstall = (FGDKPlatformChunkInstall*)FPlatformMisc::GetPlatformChunkInstall(); // note: not really safe!
	for (auto& Itr : ChunkInstall->ChunkMonitors)
	{
		XPackageChunkAvailability Availability = XPackageChunkAvailability::Unavailable;
		XPackageFindChunkAvailability( ChunkInstall->PackageIdentifier, 1, &Itr.Value->Selector, &Availability );
		UE_LOGF( LogChunkInstaller, Log, "%ls %ls", *Itr.Value->GetDebugString(), *LexToString(Availability) );
	}
	for (auto& Itr : ChunkInstall->NamedChunkMonitors)
	{
		XPackageChunkAvailability Availability = XPackageChunkAvailability::Unavailable;
		XPackageFindChunkAvailability( ChunkInstall->PackageIdentifier, 1, &Itr.Value->Selector, &Availability );
		UE_LOGF( LogChunkInstaller, Log, "%ls %ls", *Itr.Value->GetDebugString(), *LexToString(Availability) );
	}
}












FGDKPlatformChunkInstall::FGDKCustomChunkSelector::FGDKCustomChunkSelector( const TSet<FName>& NamedChunks, FGDKPlatformChunkInstall* Owner )
{
	ChunkSelectors.Reserve(NamedChunks.Num());

	for (const FName& NamedChunk : NamedChunks)
	{
		FChunkMonitor* ChunkMonitor = Owner->FindChunkMonitor(NamedChunk);
		if (ChunkMonitor != nullptr)
		{
			ChunkSelectors.Emplace( ChunkMonitor->Selector ); //note: copying the selector including a shallow copy of the tag/feature/language string. It is safe because this class is only ever created on the stack & will never outlive the chunk monitors
			StringTagList += ChunkMonitor->GetName();
		}
		else
		{
			UE_LOGF(LogChunkInstaller, Error, "Unknown Named Chunk %ls", *NamedChunk.ToString());
			StringTagList += NamedChunk.ToString() + TEXT("??");
		}
		StringTagList += TEXT(", ");

	}

	// clean up debug string
	StringTagList.RemoveFromEnd( TEXT(", ") );
}










FGDKPlatformChunkInstall::FChunkMonitor::FChunkMonitor( FGDKPlatformChunkInstall& InOwner, const XPackageChunkSelector& InSelector, XPackageChunkAvailability InInitialAvailability )
	: bInstalled(false)
	, bWasInProgress(false)
	, CachedProgress(0)
	, Selector(InSelector)
	, InstallationMonitorHandle(nullptr)
	, Owner(&InOwner)
{
	// need to take a copy of the string
	if (Selector.type != XPackageChunkSelectorType::Chunk)
	{
		// sanity check: Selector data is a union so these should all be the same otherwise the code below 
		check(Selector.feature == Selector.language);
		check(Selector.feature == Selector.tag);

		SIZE_T TagLength = FCStringAnsi::Strlen(Selector.tag);
		SelectorTag.Reset( new char[TagLength+1] );
		FCStringAnsi::Strncpy( SelectorTag.Get(), Selector.tag, TagLength + 1 );
		Selector.tag = SelectorTag.Get();
	}

	HRESULT hResult;

	// if the chunk is already Ready when this is created (on startup) it means the chunk is downloaded already
	if (InInitialAvailability == XPackageChunkAvailability::Ready)
	{
		CachedProgress = 100;
		bInstalled = true;
	}

	// create installation monitor for this chunk
	hResult = XPackageCreateInstallationMonitor( Owner->PackageIdentifier, 1, &Selector, GDK_CHUNKINSTALL_UPDATEINTERVAL_MS, FGDKAsyncTaskQueue::GetGenericQueue(), &InstallationMonitorHandle );
	UE_CLOGF( FAILED(hResult), LogChunkInstaller, Fatal, "Failed to create installation monitor for %ls. 0x%X", *GetName(), hResult );

	// register callback when package installation state changes
	auto InstallationProgressChanged = [](void* Context, XPackageInstallationMonitorHandle MonitorHandle)
	{
		FChunkMonitor* This = (FChunkMonitor*)Context;
		This->OnProgressChange();
	};
	hResult = XPackageRegisterInstallationProgressChanged( InstallationMonitorHandle, this, InstallationProgressChanged, &InstallationProgressChangedToken );
	UE_CLOGF( FAILED(hResult), LogChunkInstaller, Fatal, "Failed to register for installation callback for %ls. 0x%X", *GetName(), hResult );
}


FGDKPlatformChunkInstall::FChunkMonitor::~FChunkMonitor()
{
	XPackageUnregisterInstallationProgressChanged(InstallationMonitorHandle, InstallationProgressChangedToken, true);
	XPackageCloseInstallationMonitorHandle(InstallationMonitorHandle);
}

FString FGDKPlatformChunkInstall::FChunkMonitor::GetDebugString() const
{
	return FString::Printf(TEXT("%-32s %4d%%   %-16s "), *GetName(), CachedProgress, bInstalled ? TEXT("installed") : TEXT(""));
}

FString FGDKPlatformChunkInstall::FChunkMonitor::GetName() const
{
	switch (Selector.type)
	{
		case XPackageChunkSelectorType::Language: return FString::Printf(TEXT("Language %s"), UTF8_TO_TCHAR(Selector.language));
		case XPackageChunkSelectorType::Tag:      return FString::Printf(TEXT("Tag %s"), UTF8_TO_TCHAR(Selector.tag));
		case XPackageChunkSelectorType::Chunk:    return FString::Printf(TEXT("Chunk %d"), Selector.chunkId);
		case XPackageChunkSelectorType::Feature:  return FString::Printf(TEXT("Feature %s"), UTF8_TO_TCHAR(Selector.feature));
	}
	checkNoEntry();
	return FString();
}

void FGDKPlatformChunkInstall::FChunkMonitor::OnProgressChange()
{
	FScopeLock Lock(&Owner->ChunkDataLock);

	// cache updated chunk installation progress
	XPackageInstallationProgress Progress;
	XPackageGetInstallationProgress( InstallationMonitorHandle, &Progress );
	CachedProgress = (Progress.totalBytes > 0) ? (Progress.installedBytes * 100) / Progress.totalBytes : 0;
	UE_LOGF( LogChunkInstaller, Verbose, "%ls installation progress changed: %llu / %llu bytes (%d%%).%ls", *GetName(), Progress.installedBytes, Progress.totalBytes, CachedProgress, Progress.completed?TEXT(" completed"):TEXT(""));	

	// check installation state
	bool bComplete = false;
	if (Progress.completed && !bInstalled && Progress.totalBytes > 0)
	{
		bInstalled = true;
		bComplete = true;
	}
	else if ((bInstalled || bWasInProgress) && (Progress.installedBytes == 0) && (Progress.totalBytes != UINT64_MAX))
	{
		bInstalled = false;
		bComplete = true;
	}
	else if (Progress.completed && !bInstalled && Progress.totalBytes == 0 && Selector.type != XPackageChunkSelectorType::Chunk)
	{
		FName NamedChunk(Selector.tag);
		Owner->UnavailableChunks.Add(NamedChunk); // most likely the chunk's Device or Language doesn't match the system
	}

	// signal completion
	if (bComplete)
	{
		bWasInProgress = false;

		if (Selector.type == XPackageChunkSelectorType::Chunk)
		{
			const uint32 UnrealChunkID = ToUnrealChunkID(Selector.chunkId);
			Owner->OnChunkInstallationChanged(UnrealChunkID, bInstalled);
		}
		else
		{
			FName NamedChunk(Selector.tag);
			Owner->OnNamedChunkInstallationChanged( NamedChunk, bInstalled );
		}
	}
	else
	{
		bWasInProgress |= (Progress.installedBytes > 0);
	}

}

bool FGDKPlatformChunkInstall::FChunkMonitor::SetPriority(EChunkPriority::Type Priority)
{
	// change the priority of this chunk if we need it right away
	if (Priority == EChunkPriority::Immediate)
	{
		GDK_SCOPE_NOT_TIME_SENSITIVE(); // XPackageChangeChunkInstallOrder is not safe to call on a time-sensitive thread

		HRESULT hResult = XPackageChangeChunkInstallOrder( Owner->PackageIdentifier, 1, &Selector );
		UE_CLOGF(FAILED(hResult), LogChunkInstaller, Error, "Failed to change chunk install order for %ls - 0x%X", *GetName(), hResult );
		return SUCCEEDED(hResult);
	}

	return true;
}

float FGDKPlatformChunkInstall::FChunkMonitor::GetProgress(EChunkProgressReportingType::Type ReportType) const
{
	// make sure we can support this
	if (ReportType != EChunkProgressReportingType::PercentageComplete)
	{
		UE_LOGF(LogChunkInstaller, Error, "ChunkProgressReportType not supported: %i", (int)ReportType);
		return 0.0f;
	}

	// return the last known progress
	return (float)CachedProgress;
}

EChunkLocation::Type FGDKPlatformChunkInstall::FChunkMonitor::GetLocation() const
{
	// return the last known location
	return bInstalled ? EChunkLocation::LocalFast : EChunkLocation::NotAvailable;
}




FString LexToString( const XPackageChunkSelector& ChunkSelector )
{
	switch(ChunkSelector.type)
	{
		case XPackageChunkSelectorType::Chunk:		return FString::Printf( TEXT("Id:%d"), ChunkSelector.chunkId );
		case XPackageChunkSelectorType::Language:	return FString::Printf( TEXT("Lang:%s"), ANSI_TO_TCHAR(ChunkSelector.language) );
		case XPackageChunkSelectorType::Tag:		return FString::Printf( TEXT("Tag:%s"), ANSI_TO_TCHAR(ChunkSelector.tag) );
		case XPackageChunkSelectorType::Feature:	return FString::Printf( TEXT("Feat:%s"), ANSI_TO_TCHAR(ChunkSelector.feature) );
		default:									return FString::Printf( TEXT("Unknown type:%d"), EnumToUnderlyingType(ChunkSelector.type) );
	}
}

FString LexToString(XPackageChunkAvailability Availability)
{
	switch(Availability)
	{
		case XPackageChunkAvailability::Ready:			return TEXT("Ready");
		case XPackageChunkAvailability::Pending:		return TEXT("Pending");
		case XPackageChunkAvailability::Installable:	return TEXT("Installable");
		case XPackageChunkAvailability::Unavailable:	return TEXT("Unvailable");
		default:										return TEXT("Invalid");
	}
}

#endif //WITH_GRDK
