// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_GRDK
#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/Ticker.h"
#include "Templates/UniquePtr.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XPackage.h>
#include <atomic>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#define UE_API GDKPACKAGECHUNKINSTALL_API

// Set to 1 in your target.cs file to enable asyncronous initialization for faster engine startup
// You must call AsyncInit and wait for the callback before using any chunk install API functions
#ifndef WITH_CHUNKINSTALL_ASYNC_INIT
	#define WITH_CHUNKINSTALL_ASYNC_INIT 0
#endif

// Set to 1 in your target.cs file to use the package manifest when initializing, rather than enumerating the XPackage.
// This seems to be faster and may work around very rare issues where the XPackage enumeration callback fails.
// This may become the default once it has been proven at scale.
// (Note that when this is enabled, Named Chunks will not be available for package Languages that are not referenced in IntelligentDeliveryChunks' StageIds. Previously these would have been treated as if they were installed.)
#ifndef USE_GDK_PACKAGE_MANIFEST_INIT
	#define USE_GDK_PACKAGE_MANIFEST_INIT 0
#endif

/**
 * GDK implementation of FGenericPlatformChunkInstall with support for Recipes & Features
 */
class FGDKPlatformChunkInstall : public FGenericPlatformChunkInstall
{
public:
	UE_API FGDKPlatformChunkInstall();
	UE_API virtual ~FGDKPlatformChunkInstall();

	//IPlatformChunkInstall implementation
#if WITH_CHUNKINSTALL_ASYNC_INIT
	UE_API virtual void AsyncInit( TFunction<void(bool/*bSuccess*/)> OnInitComplete ) override;
#endif
	UE_API virtual bool IsAvailable() const override;
	UE_API virtual EChunkLocation::Type GetPakchunkLocation(int32 PakchunkIndex) override;
	virtual bool GetProgressReportingTypeSupported(EChunkProgressReportingType::Type ReportType) override { return ReportType == EChunkProgressReportingType::PercentageComplete; }
	UE_API virtual float GetChunkProgress( uint32 PakchunkIndex, EChunkProgressReportingType::Type ReportType ) override;
	virtual EChunkInstallSpeed::Type GetInstallSpeed() override { return EChunkInstallSpeed::Slow; }
	virtual bool SetInstallSpeed( EChunkInstallSpeed::Type InstallSpeed ) override { return false; }
	UE_API virtual bool PrioritizePakchunk(int32 PakchunkIndex, EChunkPriority::Type Priority) override;
	virtual bool DebugStartNextChunk() { return false; }
	UE_API virtual void ExternalNotifyChunkAvailable(uint32 PakchunkIndex) override;

	virtual bool SupportsNamedChunkInstall() const override { return true; }
	UE_API virtual bool IsNamedChunkInProgress(const FName NamedChunk) override;
	UE_API virtual bool InstallNamedChunks(const TArrayView<const FName>& NamedChunks) override;
	UE_API virtual bool UninstallNamedChunks(const TArrayView<const FName>& NamedChunks) override;
	UE_API virtual EChunkLocation::Type GetNamedChunkLocation(const FName NamedChunk) override;
	UE_API virtual float GetNamedChunkProgress(const FName NamedChunk, EChunkProgressReportingType::Type ReportType) override;
	UE_API virtual bool PrioritizeNamedChunk(const FName NamedChunk, EChunkPriority::Type Priority) override;
	UE_API virtual bool CancelNamedChunksInstall(const TArrayView<const FName>& NamedChunks) override;
	UE_API virtual ENamedChunkType GetNamedChunkType(const FName NamedChunk) const override;
	UE_API virtual TArray<FName> GetNamedChunksByType(ENamedChunkType NamedChunkType) const override;

	virtual bool SupportsBundleSource() const override { return true; }

	UE_API virtual bool SetAutoPakMountingEnabled( bool bEnabled ) override;
	UE_API virtual bool GetPakFilesInNamedChunk( const FName NamedChunk, TArray<FString>& OutFilesInChunk) const override;
	UE_API virtual bool GetNamedChunkInstallationStatus( const FName NamedChunk, FChunkInstallationStatusDetail& OutChunkStatusDetail ) const override;
	UE_API virtual bool IsNamedChunkForCurrentLocale( const FName NamedChunk ) const override;

	// helper functions
	static UE_API void DebugDumpChunkState();

private:
#if WITH_CHUNKINSTALL_ASYNC_INIT
	std::atomic<bool> bIsInitialized = false;
#endif

	//IPlatformChunkInstall implementation
	UE_API virtual bool PrioritizeChunk(uint32 UnrealChunkID, EChunkPriority::Type Priority) override;
	UE_API virtual EChunkLocation::Type GetChunkLocation(uint32 UnrealChunkID) override;

	// common chunk install functionality
	bool bAutoMountPaks = true;
	UE_API bool MountPaks(uint32 UnrealChunkID);
	UE_API bool UnMountPaks(uint32 UnrealChunkID);

	// package data
	bool bIsPackaged;
	bool bPackageHasFeatures;
    char PackageIdentifier[XPACKAGE_IDENTIFIER_MAX_LENGTH];

	// cached information about a chunk
	struct FChunkMonitor
	{
		FChunkMonitor( FGDKPlatformChunkInstall& Owner, const XPackageChunkSelector& InSelector, XPackageChunkAvailability InInitialAvailability );
		~FChunkMonitor();

		FString GetDebugString() const;
		FString GetName() const;
		void OnProgressChange();

		bool SetPriority(EChunkPriority::Type Priority);
		float GetProgress(EChunkProgressReportingType::Type ReportType) const;
		EChunkLocation::Type GetLocation() const;

		bool bInstalled;
		bool bWasInProgress;
		uint8 CachedProgress; //0-100
		TUniquePtr<char[]> SelectorTag;
		XPackageChunkSelector Selector;
		XPackageInstallationMonitorHandle InstallationMonitorHandle;
		XTaskQueueRegistrationToken InstallationProgressChangedToken;
		FGDKPlatformChunkInstall* Owner;
	};

	// chunk data
	TArray<FName> OnDemandChunks;
	TArray<FName> LanguageChunks;
	TMap<uint32,TUniquePtr<FChunkMonitor>> ChunkMonitors;
	TMap<FName,TUniquePtr<FChunkMonitor>> NamedChunkMonitors;
	TMultiMap<FName,int32> InitialNamedChunksMap;
	TSet<FName> SecondaryLanguageNamedChunks;
	TSet<FName> UnavailableChunks;
	FCriticalSection ChunkDataLock;

	// chunk data functions
#if USE_GDK_PACKAGE_MANIFEST_INIT
	UE_API bool CreateChunkDataFromPackageManifest();
#else
	UE_API void CreateChunkDataForPackage(const XPackageDetails& Details);
#endif
	UE_API void OnNamedChunkInstallationChanged(const FName NamedChunk, bool bIsInstalled);
	UE_API void OnChunkInstallationChanged(uint32 UnrealChunkID, bool bIsInstalled);
	UE_API FChunkMonitor* FindChunkMonitor(uint32 UnrealChunkID);
	UE_API FChunkMonitor* FindChunkMonitor(const FName NamedChunk);
	const FChunkMonitor* FindChunkMonitor(uint32 UnrealChunkID) const   { return const_cast<FGDKPlatformChunkInstall*>(this)->FindChunkMonitor(UnrealChunkID); }
	const FChunkMonitor* FindChunkMonitor(const FName NamedChunk) const { return const_cast<FGDKPlatformChunkInstall*>(this)->FindChunkMonitor(NamedChunk); }

	// ticker & deferred operations
	UE_API void AddToTicker();
	UE_API void RemoveFromTicker();
	UE_API bool Tick( float DeltaSeconds );
	FTSTicker::FDelegateHandle TickHandle;
	TSet<FName> PendingInstallChunks;
	TSet<FName> PendingUninstallChunks;
	TSet<FName> CurrentUninstallingChunks;

	// helper class for creating an GDK XPackageChunkSelector array from an FName array
	class FGDKCustomChunkSelector
	{
	public:
		FGDKCustomChunkSelector( const TSet<FName>& NamedChunks, FGDKPlatformChunkInstall* Owner );
		inline const FString& ToString() const					{ return StringTagList; }
		inline const uint32 Num() const							{ return ChunkSelectors.Num(); }
		inline XPackageChunkSelector* Get()						{ return ChunkSelectors.GetData(); }
	
	private:
		FString							StringTagList;		//comma-separated list of tags, for logging only
		TArray<XPackageChunkSelector>	ChunkSelectors;		//chunk selectors for the chunk tag ids
	};
};


FString UE_API LexToString(const XPackageChunkSelector& ChunkSelector);
FString UE_API LexToString(XPackageChunkAvailability Availability);

#undef UE_API

#else
// dummy... needed?
//#include "GenericPlatform/GenericPlatformChunkInstall.h"
//typedef FGenericPlatformChunkInstall FGDKPlatformChunkInstall;

#endif //WITH_GRDK
